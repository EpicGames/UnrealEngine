# Adding New Metric Types to Performance Trends

This guide walks through adding a new metric type to the Performance Trends dashboard, covering both server-side (C#) and client-side (TypeScript/React) implementation.

## Overview

The Performance Trends system uses a registry-based architecture where metric types are:
1. **Server-side**: Queried via `AbstractPerformanceSummaryTrendHandler` implementations
2. **Client-side**: Visualized via `IMetricViewGenerator<T>` implementations

Both sides are linked by a shared `metricType` string identifier.

---

## Server-Side Implementation

### Step 1: Define the Telemetry Record

Create a new record class in `EpicGames.Analytics.PerformanceTrends/`:

```csharp
// MemoryStatsTelemetryRecord.cs

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics.PerformanceTrends;

namespace EpicGames.Analytics
{
    /// <summary>
    /// Telemetry record for memory performance metrics.
    /// </summary>
    [AnalyticsTableGen]
    [Table("schema.memory_stats_summary")]
    public record MemoryStatsTelemetryRecord : PerformanceTrendTelemetry
    {
        [Column("heap_used_mb")]
        public float HeapUsedMb { get; init; }

        [Column("texture_memory_mb")]
        public float TextureMemoryMb { get; init; }

        [Column("mesh_memory_mb")]
        public float MeshMemoryMb { get; init; }

        [Column("audio_memory_mb")]
        public float AudioMemoryMb { get; init; }

        [Column("total_allocated_mb")]
        public float TotalAllocatedMb { get; init; }

        public MemoryStatsTelemetryRecord() : base() { }
    }
}
```

> **Note**: The `[AnalyticsTableGen]` attribute generates `EpicGames_Analytics_MemoryStatsTelemetryRecordGen` with type-safe column constants and Dapper mappings.

### Step 2: Create the Handler

Create a handler in `HordeServer.Epic.EpicSandbox/PerformanceTrends/`:

```csharp
// MemoryStatsPerformanceSummaryHandler.cs

using System.Data;
using System.Data.Common;
using System.Data.Odbc;
using EpicGames.Analytics;
using EpicGames.Analytics.Generated;
using EpicGames.Analytics.PerformanceTrends;
using HordeServer.Analytics;
using Microsoft.Extensions.Logging;

namespace HordeServer.LicenseePromotionCandidate.PerformanceTrends
{
    /// <summary>
    /// Summary handler for memory statistics metrics.
    /// </summary>
    public class MemoryStatsPerformanceSummaryHandler : AbstractPerformanceSummaryTrendHandler
    {
        // Register Dapper column mappings at startup
        static MemoryStatsPerformanceSummaryHandler()
        {
            ColumnHandlers.RegisterDapperColumnMapping<MemoryStatsTelemetryRecord>();
        }

        /// <inheritdoc/>
        public override string PerformanceSummaryType => "MemoryStats";

        /// <inheritdoc/>
        public override async Task<IEnumerable<PerformanceTrendTelemetry>> ProcessGeneralMetricRequest(
            OdbcConnection dbConnection,
            PerformanceTrendFilter filter,
            ILogger logger,
            CancellationToken cancellationToken)
        {
            List<MemoryStatsTelemetryRecord> results = [];

            using (OdbcCommand cmd = dbConnection.CreateCommand())
            {
                string whereSql = BuildWhereClause(filter, cmd);
                string sql = $@"
                    SELECT
                        {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.HeapUsedMb},
                        {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TextureMemoryMb},
                        {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.MeshMemoryMb},
                        {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.AudioMemoryMb},
                        {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TotalAllocatedMb},
                        {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.SummaryName},
                        {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.TestName},
                        {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitId}
                    FROM {EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TableName}
                    {whereSql}
                    ORDER BY {EpicGames_Analytics_PerformanceTrends_PerformanceTrendTelemetryGen.CommitId} DESC
                    LIMIT {filter.RecordCount}";

#pragma warning disable CA2100
                cmd.CommandText = sql;
#pragma warning restore CA2100
                cmd.CommandType = CommandType.Text;

                using (DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken))
                {
                    results = reader.Parse<MemoryStatsTelemetryRecord>().AsList();
                    await reader.CloseAsync();
                }
            }

            return results;
        }

        /// <inheritdoc/>
        protected override string GetTableName()
        {
            return EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TableName;
        }

        /// <inheritdoc/>
        protected override IReadOnlyList<PerformanceTrendTelemetry> ParseTypeFromReader(DbDataReader reader)
        {
            return reader.Parse<MemoryStatsTelemetryRecord>().ToList();
        }

        private static string BuildWhereClause(PerformanceTrendFilter filter, OdbcCommand cmd)
        {
            List<string> where = new();

            if (!string.IsNullOrEmpty(filter.TestProject))
            {
                where.Add($"{EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TestName} = ?");
                var param = cmd.CreateParameter();
                param.OdbcType = OdbcType.VarChar;
                param.Value = filter.TestProject;
                cmd.Parameters.Add(param);
            }

            if (!string.IsNullOrEmpty(filter.TestIdentity))
            {
                where.Add($"{EpicGames_Analytics_MemoryStatsTelemetryRecordGen.TestIdentity} = ?");
                var param = cmd.CreateParameter();
                param.OdbcType = OdbcType.VarChar;
                param.Value = filter.TestIdentity;
                cmd.Parameters.Add(param);
            }

            return where.Count > 0 ? "WHERE " + string.Join(" AND ", where) : string.Empty;
        }
    }
}
```

### Step 3: Register in Dependency Injection

In your service configuration:

```csharp
services.AddSingleton<AbstractPerformanceSummaryTrendHandler, MemoryStatsPerformanceSummaryHandler>();
```

---

## Client-Side Implementation

### Step 4: Define the Metric Data Class

Create in `PerformanceTrends/metrictypes/`:

```typescript
// MemoryStatsData.ts

import { MemoryStatsViewGenerator } from "../viewgenerators/MemoryStatsViewGenerator";
import { ISummaryMetric, MetricTypeRegistry, PerformanceTrendContext } from "./PerformanceTrendsTypes";

/**
 * Memory statistics metric data structure.
 */
export class MemoryStatsData extends PerformanceTrendContext implements ISummaryMetric {

    // Properties must match server-side MemoryStatsTelemetryRecord
    heapUsedMb: number;
    textureMemoryMb: number;
    meshMemoryMb: number;
    audioMemoryMb: number;
    totalAllocatedMb: number;

    /**
     * The metric type identifier. Must match server-side PerformanceSummaryType.
     */
    static readonly metricType = "MemoryStats";

    constructor() {
        super();
    }
}

// Register with the metric type registry
MetricTypeRegistry.register(MemoryStatsData, new MemoryStatsViewGenerator());
```

> **Important**: The `static readonly metricType` must exactly match the server-side `PerformanceSummaryType`.

### Step 5: Create the View Generator

Create in `PerformanceTrends/viewgenerators/`:

```typescript
// MemoryStatsViewGenerator.tsx

import { observer } from "mobx-react-lite";
import { graphColors } from "hordePlugins/analytics/telemetryData";
import { LineGraph } from "../components/LineGraphComponent";
import { MemoryStatsData } from "../metrictypes/MemoryStatsData";
import { IMetricView, IMetricViewGenerator } from "./PerformanceTrendRenderTypes";
import { PerformanceTrendUIOptionsState } from "../filters/PerformanceTrendUIOptionsState";

/**
 * View generator for memory statistics metrics.
 */
export class MemoryStatsViewGenerator implements IMetricViewGenerator<MemoryStatsData> {

    getViews(metrics: MemoryStatsData[], uiState: PerformanceTrendUIOptionsState): IMetricView[] {

        // Extract data series
        const heapUsed = metrics.map(m => m.heapUsedMb);
        const textureMemory = metrics.map(m => m.textureMemoryMb);
        const meshMemory = metrics.map(m => m.meshMemoryMb);
        const totalAllocated = metrics.map(m => m.totalAllocatedMb);

        // Calculate averages for display
        const avgHeap = heapUsed.length > 0
            ? heapUsed.reduce((a, b) => a + b, 0) / heapUsed.length
            : 0;

        // Define views
        const HeapMemoryView = observer(({ width, height }: { width: number; height: number }) => {
            const { curveGraphLine } = uiState;

            return (
                <div>
                    <div>
                        <span>Heap Memory Usage</span><br />
                        <span>(Avg) {avgHeap.toFixed(1)} MB</span>
                    </div>
                    <LineGraph
                        data={[heapUsed]}
                        legendLabels={["Heap Used"]}
                        colorLabels={[graphColors[0]]}
                        xAxisLabel="Time/Index"
                        yAxisLabel="Memory (MB)"
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={curveGraphLine}
                    />
                </div>
            );
        });

        const MemoryBreakdownView = observer(({ width, height }: { width: number; height: number }) => {
            const { curveGraphLine } = uiState;

            return (
                <div>
                    <div>
                        <span>Memory Breakdown by Type</span>
                    </div>
                    <LineGraph
                        data={[textureMemory, meshMemory, heapUsed]}
                        legendLabels={["Textures", "Meshes", "Heap"]}
                        xAxisLabel="Time/Index"
                        yAxisLabel="Memory (MB)"
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={curveGraphLine}
                    />
                </div>
            );
        });

        const TotalAllocationView = observer(({ width, height }: { width: number; height: number }) => {
            const { curveGraphLine } = uiState;

            return (
                <div>
                    <div>
                        <span>Total Memory Allocated</span>
                    </div>
                    <LineGraph
                        data={[totalAllocated]}
                        legendLabels={["Total Allocated"]}
                        colorLabels={[graphColors[5]]}
                        xAxisLabel="Time/Index"
                        yAxisLabel="Memory (MB)"
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={curveGraphLine}
                    />
                </div>
            );
        });

        return [
            { label: "Heap Memory", Render: HeapMemoryView },
            { label: "Memory Breakdown", Render: MemoryBreakdownView },
            { label: "Total Allocation", Render: TotalAllocationView },
        ];
    }
}
```

### Step 6: Import for Side-Effect Registration

Ensure the metric data file is imported so registration runs:

```typescript
// In PerformanceTrendView.tsx or a central imports file
import "./PerformanceTrends/metrictypes/MemoryStatsData";  // Side-effect: registers metric
```

---

## Architecture Summary

```
┌─────────────────────────────────────────────────────────────────┐
│                         SERVER                                  │
├─────────────────────────────────────────────────────────────────┤
│  [AnalyticsTableGen]              → Generates column constants  │
│  MemoryStatsTelemetryRecord       → Data schema                 │
│  MemoryStatsPerformanceSummaryHandler → Query logic             │
│  DI Registration                  → Service discovery           │
└─────────────────────────────────────────────────────────────────┘
                              │
                              │  metricType = "MemoryStats"
                              ▼
┌─────────────────────────────────────────────────────────────────┐
│                         CLIENT                                  │
├─────────────────────────────────────────────────────────────────┤
│  MemoryStatsData                  → TypeScript model            │
│  MemoryStatsViewGenerator         → Visualization components    │
│  MetricTypeRegistry.register()    → Single registration call    │
└─────────────────────────────────────────────────────────────────┘
```

---

## Checklist

### Server-Side

- [ ] Create telemetry record with `[AnalyticsTableGen]` attribute
- [ ] Define `[Table]` and `[Column]` attributes for ORM mapping
- [ ] Create handler extending `AbstractPerformanceSummaryTrendHandler`
- [ ] Implement `PerformanceSummaryType` property (the type identifier)
- [ ] Implement `ProcessGeneralMetricRequest` with custom query logic
- [ ] Register Dapper column mapping in static constructor
- [ ] Register handler in DI container

### Client-Side

- [ ] Create data class extending `PerformanceTrendContext`
- [ ] Add `static readonly metricType` matching server-side
- [ ] Define properties matching server-side record fields
- [ ] Create view generator implementing `IMetricViewGenerator<T>`
- [ ] Return `IMetricView[]` from `getViews()` for each visualization
- [ ] Call `MetricTypeRegistry.register()` at module load
- [ ] Import the data file to trigger registration

### Verification

- [ ] `metricType` strings match exactly between server and client
- [ ] Property names match between `TelemetryRecord` and `Data` class
- [ ] Build passes on both server (`dotnet build`) and client (`npm run build`)
- [ ] New metric type appears in dashboard dropdown
- [ ] Data loads and visualizations render correctly
